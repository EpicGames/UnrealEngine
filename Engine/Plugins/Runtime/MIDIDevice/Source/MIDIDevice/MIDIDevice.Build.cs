// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MIDIDevice : ModuleRules
	{
        public MIDIDevice(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
                    "Core",
					"CoreUObject",
					"Engine",
				}
			);
			AddEngineThirdPartyPrivateStaticDependencies(Target, "portmidi");
		}
	}
}
