// Copyright Epic Games, Inc. All Rights Reserved.

#include "MIDIDeviceController.h"
#include "MIDIDeviceLog.h"
#include "portmidi.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MIDIDeviceController)


#define LOCTEXT_NAMESPACE "MIDIDeviceController"


UMIDIDeviceController::~UMIDIDeviceController()
{
	// Clean everything up before we're garbage collected
	UMIDIDeviceController::ShutdownDevice();
}


// todo midi: Add SetFilter and SetChannelMask support (better performance)

void UMIDIDeviceController::StartupDevice( const int32 InitDeviceID, const int32 InitMIDIBufferSize, bool& bOutWasSuccessful )
{
	bOutWasSuccessful = false;

	this->DeviceID = InitDeviceID;
	this->PMMIDIInputStream = nullptr;
	this->MIDIBufferSize = 0;

	const PmDeviceID PMDeviceID = DeviceID;

	if(InitMIDIBufferSize > 0)
	{
		const PmDeviceInfo* PMDeviceInfo = Pm_GetDeviceInfo(PMDeviceID);
		if(PMDeviceInfo != nullptr)
		{
			// Is the device already in use?  If so, spit out a warning
			if(PMDeviceInfo->opened != 0)
			{
				UE_LOGF(LogMIDIDevice, Warning, "Warning while creating a MIDI device controller:  PortMidi reports that device ID %i (%ls) is already in use.", PMDeviceID, ANSI_TO_TCHAR(PMDeviceInfo->name));
			}

			// Make sure the device is setup for input/output
			if( PMDeviceInfo->input == 0 )
			{
				UE_LOGF(LogMIDIDevice, Warning, "Warning while creating a MIDI device controller:  PortMidi reports that device ID %i (%ls) does is not setup to transmit MIDI data.", PMDeviceID, ANSI_TO_TCHAR(PMDeviceInfo->name));
			}

			// @todo midi: Add options for timing/latency (see timeproc, and pm_Synchronize)

			const PmError PMError = Pm_OpenInput( &this->PMMIDIInputStream, PMDeviceID, NULL, MIDIBufferSize, NULL, NULL );
			if(PMError == pmNoError)
			{
				check(this->PMMIDIInputStream != nullptr);

				this->DeviceName = ANSI_TO_TCHAR(PMDeviceInfo->name);
				this->MIDIBufferSize = InitMIDIBufferSize;

				// Good to go!
				bOutWasSuccessful = true;
			}
			else
			{
				this->PMMIDIInputStream = nullptr;
				const FString ErrorText = MIDIDeviceInternal::ParsePmError(PMError);
				UE_LOGF(LogMIDIDevice, Error, "Unable to open input connection to MIDI device ID %i (%ls) (PortMidi error: %ls).", PMDeviceID, ANSI_TO_TCHAR(PMDeviceInfo->name), *ErrorText);
			}
		}
		else
		{
			UE_LOGF(LogMIDIDevice, Error, "Unable to query information about MIDI device (PortMidi device ID: %i).", PMDeviceID);
		}
	}
	else
	{
		UE_LOGF(LogMIDIDevice, Error, "The specified MIDI Buffer Size must be greater than zero.");
	}
}

void UMIDIDeviceController::ShutdownDevice()
{
	const PmDeviceID PMDeviceID = this->DeviceID;

	if(this->PMMIDIInputStream != nullptr)
	{
		Pm_Close(this->PMMIDIInputStream);
		this->PMMIDIInputStream = nullptr;
	}
}


void UMIDIDeviceController::ProcessIncomingMIDIEvents()
{
	if(this->PMMIDIInputStream != nullptr)
	{
		const PmDeviceID PMDeviceID = this->DeviceID;

		// Static that we'll copy event data to every time.  This stuff isn't multi-threaded right now, so this is fine.
		static TArray<PmEvent> PMMIDIEvents;
		PMMIDIEvents.SetNum(MIDIBufferSize, EAllowShrinking::No);

		const int32 PMEventCount = Pm_Read(this->PMMIDIInputStream, PMMIDIEvents.GetData(), PMMIDIEvents.Num());
 
		// if Pm_Read returns a negative result, it's an error
		if(PMEventCount < 0)
		{
			const FString ErrorText = MIDIDeviceInternal::ParsePmError(static_cast<const PmError>(PMEventCount));
			UE_LOGF(LogMIDIDevice, Error, "Encountered an error when processing incoming MIDI events for device ID %i (%ls) (PortMidi error: %ls).", this->DeviceID, *this->DeviceName, *ErrorText);
		}
		else
		{
			for(int32 PMEventIndex = 0; PMEventIndex < PMEventCount; ++PMEventIndex)
			{
				const PmEvent& PMEvent = PMMIDIEvents[PMEventIndex];
				const int32 PMTimestamp = PMEvent.timestamp;

				const PmMessage& PMMessage = PMEvent.message;

				const int PMMessageStatus = Pm_MessageStatus(PMMessage);
				const int PMMessageData1 = Pm_MessageData1( PMMessage );
				const int PMMessageData2 = Pm_MessageData2( PMMessage );
				const int32 PMType = (PMMessageStatus & 0xF0) >> 4;
				const int32 PMChannel = (PMMessageStatus % 16) + 1;

				// Send our event
				{
					const int32 Timestamp = PMTimestamp;
					EMIDIEventType EventType = EMIDIEventType::Unknown;

					EMIDIEventType PossibleEventType = (EMIDIEventType)PMType;
					switch(PossibleEventType)
					{
					case EMIDIEventType::NoteOff:
					case EMIDIEventType::NoteOn:
					case EMIDIEventType::NoteAfterTouch:
					case EMIDIEventType::ControlChange:
					case EMIDIEventType::ProgramChange:
					case EMIDIEventType::ChannelAfterTouch:
					case EMIDIEventType::PitchBend:
						EventType = PossibleEventType;	// NOTE: Our values match up, so we can just cast/assign
						break;
					}
					
					const int32 Channel = PMChannel;
					const int32 ControlID = PMMessageData1;
					const int32 Velocity = PMMessageData2;
					const int32 RawEventType = PMType;

					this->OnMIDIEvent.Broadcast(this, Timestamp, EventType, Channel, ControlID, Velocity, RawEventType);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

