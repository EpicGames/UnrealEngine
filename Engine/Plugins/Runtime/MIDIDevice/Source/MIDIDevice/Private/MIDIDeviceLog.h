// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include <portmidi.h>

DECLARE_LOG_CATEGORY_EXTERN(LogMIDIDevice, Verbose, All);

namespace MIDIDeviceInternal
{
	// Returns error text for the given PmError
	static FString ParsePmError(const PmError& InError)
	{
		FString ErrorText = ANSI_TO_TCHAR(Pm_GetErrorText(InError));
		if(InError == pmHostError)
		{
			char ErrorTextBuffer[1024];
			Pm_GetHostErrorText(ErrorTextBuffer, 1024);
			ErrorText = ANSI_TO_TCHAR(ErrorTextBuffer);
		}
		return ErrorText;
	}
}
