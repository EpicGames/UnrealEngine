// Copyright Epic Games, Inc. All Rights Reserved.

#include "MIDIDeviceManager.h"
#include "MIDIDeviceController.h"
#include "MIDIDeviceControllerBase.h"
#include "MIDIDeviceLog.h"
#include "portmidi.h"
#include "UObject/UObjectIterator.h"
#include "UObject/ScriptInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MIDIDeviceManager)

#define LOCTEXT_NAMESPACE "MIDIDeviceManager"


bool UMIDIDeviceManager::bIsInitialized = false;
TArray<FMIDIDeviceInfo> UMIDIDeviceManager::MIDIInputDevicesInfo;
TArray<FMIDIDeviceInfo> UMIDIDeviceManager::MIDIOutputDevicesInfo;


void UMIDIDeviceManager::StartupMIDIDeviceManager()
{
	if (ensure(!bIsInitialized))
	{
		const PmError PMError = Pm_Initialize();
		if (PMError == pmNoError)
		{
			bIsInitialized = true;
		}
		else
		{
			const FString ErrorText = MIDIDeviceInternal::ParsePmError(PMError);
			UE_LOGF(LogMIDIDevice, Error, "Unable to open initialize the MIDI device manager (PortMidi error: %ls).  You won't be use MIDI features in this session.", *ErrorText);
		}
	}
}


void UMIDIDeviceManager::ShutdownMIDIDeviceManager()
{
	if(bIsInitialized)
	{
		bIsInitialized = false;

		// Kill any open connections
		for(TObjectIterator<UMIDIDeviceControllerBase> ControllerIter; ControllerIter; ++ControllerIter)
		{
			UMIDIDeviceControllerBase* MIDIDeviceController = *ControllerIter;
			if(MIDIDeviceController != nullptr && !IsValidChecked(MIDIDeviceController))
			{
				MIDIDeviceController->ShutdownDevice();
			}
		}

		Pm_Terminate();
	}
}


void UMIDIDeviceManager::ProcessMIDIEvents()
{
	if(bIsInitialized)
	{
		// @todo midi perf: Should we cache weak pointers instead of using TObjectIterator every frame?
		for(TObjectIterator<UMIDIDeviceControllerBase> ControllerIter; ControllerIter; ++ControllerIter)
		{
			UMIDIDeviceControllerBase* MIDIDeviceController = *ControllerIter;
			if(MIDIDeviceController != nullptr && IsValidChecked(MIDIDeviceController))
			{
				MIDIDeviceController->ProcessIncomingMIDIEvents();
			}
		}
	}
}


void UMIDIDeviceManager::ReinitializeDeviceManager()
{
	ShutdownMIDIDeviceManager();
	StartupMIDIDeviceManager();

	// Re-fetch devices in case names or id's have changed
	TArray<FFoundMIDIDevice> FoundDevices;
	FindMIDIDevicesInternal(FoundDevices); // use FindMIDIDevicesInternal to avoid calling ReinitializeDeviceManager/recursion

	// Re-initialize any existing controllers
	for(TObjectIterator<UMIDIDeviceControllerBase> ControllerIter; ControllerIter; ++ControllerIter)
	{
		UMIDIDeviceControllerBase* MIDIDeviceController = *ControllerIter;
		if(MIDIDeviceController != nullptr && IsValidChecked(MIDIDeviceController))
		{
			FString ExistingDeviceName = MIDIDeviceController->GetDeviceName();

			// Attempt to find from NEW devices by PREVIOUS name
			FFoundMIDIDevice* FoundDevice = FoundDevices.FindByPredicate([&](const FFoundMIDIDevice& InFoundDevice)
			{
				return InFoundDevice.DeviceName.Equals(ExistingDeviceName);
			});

			bool bStartedSuccessfully = false;
			// If found, re-initialize with new id (might be same as old)
			if(FoundDevice)
			{
				MIDIDeviceController->StartupDevice(FoundDevice->DeviceID, MIDIDeviceController->GetMIDIBufferSize(), bStartedSuccessfully);
			}

			// If device not found in NEW list, or startup failed
			if (!bStartedSuccessfully)
			{
				// Kill it
				MIDIDeviceController->MarkAsGarbage();
				MIDIDeviceController = nullptr;

				UE_LOGF(LogMIDIDevice, Warning, "MIDI Device Controller re-initialization failed.");
			}
		}
	}
}


void UMIDIDeviceManager::FindMIDIDevicesInternal(TArray<FFoundMIDIDevice>& OutMIDIDevices)
{
	if(bIsInitialized)
	{
		// Figure out what the system default input and output devices are, so we can relay that information
		const PmDeviceID DefaultInputPMDeviceID = Pm_GetDefaultInputDeviceID();
		const PmDeviceID DefaultOutputPMDeviceID = Pm_GetDefaultOutputDeviceID();

		const int PMDeviceCount = Pm_CountDevices();
		for(PmDeviceID PMDeviceID = 0; PMDeviceID < PMDeviceCount; ++PMDeviceID)
		{
			const PmDeviceInfo* PMDeviceInfo = Pm_GetDeviceInfo(PMDeviceID);
			if(PMDeviceInfo != nullptr)
			{
				FFoundMIDIDevice& FoundMIDIDevice = *new(OutMIDIDevices) FFoundMIDIDevice();

				FoundMIDIDevice.DeviceID = PMDeviceID;
				FoundMIDIDevice.DeviceName = ANSI_TO_TCHAR(PMDeviceInfo->name);
				FoundMIDIDevice.bCanReceiveFrom = PMDeviceInfo->input != 0;
				FoundMIDIDevice.bCanSendTo = PMDeviceInfo->output != 0;
				FoundMIDIDevice.bIsAlreadyInUse = PMDeviceInfo->opened != 0;
				FoundMIDIDevice.bIsDefaultInputDevice = (PMDeviceID == DefaultInputPMDeviceID);
				FoundMIDIDevice.bIsDefaultOutputDevice = (PMDeviceID == DefaultOutputPMDeviceID);
			}
			else
			{
				UE_LOGF(LogMIDIDevice, Error, "Unable to query information about MIDI device (PortMidi device ID: %i).  This device won't be available for input or output.", PMDeviceID);
			}
		}
	}
	else
	{
		UE_LOGF(LogMIDIDevice, Warning, "Find MIDI Devices cannot be used because the MIDI device manager failed to initialize.  Check earlier in the log to see why.");
	}
}

void UMIDIDeviceManager::ShutDownAllMIDIDevices()
{
	for (TObjectIterator<UMIDIDeviceControllerBase> ControllerIter; ControllerIter; ++ControllerIter)
	{
		UMIDIDeviceControllerBase* MIDIDeviceController = *ControllerIter;
		if (IsValid(MIDIDeviceController))
		{
			MIDIDeviceController->ShutdownDevice();
		}
	}
}

void UMIDIDeviceManager::FindMIDIDevices(TArray<FFoundMIDIDevice>& OutMIDIDevices)
{
	OutMIDIDevices.Reset();

	// re-initialize to get updated device list
	if(bIsInitialized)
	{
		ReinitializeDeviceManager();
	}
	
	FindMIDIDevicesInternal(OutMIDIDevices);
}


void UMIDIDeviceManager::FindAllMIDIDeviceInfoInternal(TArray<FMIDIDeviceInfo>& OutMIDIInputDevicesInfo, TArray<FMIDIDeviceInfo>& OutMIDIOutputDevicesInfo)
{
	OutMIDIInputDevicesInfo.Reset();
	OutMIDIOutputDevicesInfo.Reset();

	if (bIsInitialized)
	{
		// Figure out what the system default input and output devices are, so we can relay that information
		const PmDeviceID DefaultInputPMDeviceID = Pm_GetDefaultInputDeviceID();
		const PmDeviceID DefaultOutputPMDeviceID = Pm_GetDefaultOutputDeviceID();

		const int PMDeviceCount = Pm_CountDevices();
		for (PmDeviceID PMDeviceID = 0; PMDeviceID < PMDeviceCount; ++PMDeviceID)
		{
			const PmDeviceInfo* PMDeviceInfo = Pm_GetDeviceInfo(PMDeviceID);
			if (PMDeviceInfo != nullptr)
			{
				if (PMDeviceInfo->input != 0)
				{
					FMIDIDeviceInfo& FoundMIDIInputDeviceInfo = *new(OutMIDIInputDevicesInfo) FMIDIDeviceInfo();
					FoundMIDIInputDeviceInfo.DeviceID = PMDeviceID;
					FoundMIDIInputDeviceInfo.DeviceName = ANSI_TO_TCHAR(PMDeviceInfo->name);
					FoundMIDIInputDeviceInfo.bIsAlreadyInUse = PMDeviceInfo->opened != 0;
					FoundMIDIInputDeviceInfo.bIsDefaultDevice = (PMDeviceID == DefaultInputPMDeviceID);
				}
				else if (PMDeviceInfo->output != 0)
				{
					FMIDIDeviceInfo& FoundMIDIOutputDeviceInfo = *new(OutMIDIOutputDevicesInfo) FMIDIDeviceInfo();
					FoundMIDIOutputDeviceInfo.DeviceID = PMDeviceID;
					FoundMIDIOutputDeviceInfo.DeviceName = ANSI_TO_TCHAR(PMDeviceInfo->name);
					FoundMIDIOutputDeviceInfo.bIsAlreadyInUse = PMDeviceInfo->opened != 0;
					FoundMIDIOutputDeviceInfo.bIsDefaultDevice = (PMDeviceID == DefaultInputPMDeviceID);
				}

				MIDIInputDevicesInfo = OutMIDIInputDevicesInfo;
				MIDIOutputDevicesInfo = OutMIDIOutputDevicesInfo;
			}
			else
			{
				UE_LOGF(LogMIDIDevice, Error, "Unable to query information about MIDI device (PortMidi device ID: %i).  This device won't be available for input or output.", PMDeviceID);
			}
		}
	}
	else
	{
		UE_LOGF(LogMIDIDevice, Warning, "Find MIDI Devices cannot be used because the MIDI device manager failed to initialize.  Check earlier in the log to see why.");
	}
}


void UMIDIDeviceManager::FindAllMIDIDeviceInfo(TArray<FMIDIDeviceInfo>& OutMIDIInputDevicesInfo, TArray<FMIDIDeviceInfo>& OutMIDIOutputDevicesInfo)
{
	// re-initialize to get updated device list
	if(bIsInitialized)
	{
		ReinitializeDeviceManager();
	}
	FindAllMIDIDeviceInfoInternal(OutMIDIInputDevicesInfo, OutMIDIOutputDevicesInfo);
}

void UMIDIDeviceManager::GetMIDIInputDeviceIDByName(const FString DeviceName, int32& DeviceID)
{
	if (bIsInitialized)
	{
		for (int32 i = 0; i < MIDIInputDevicesInfo.Num(); ++i)
		{
			const FMIDIDeviceInfo& MIDIInputDevice = MIDIInputDevicesInfo[i];

			if (MIDIInputDevice.DeviceName == DeviceName)
			{
				DeviceID = MIDIInputDevice.DeviceID;
			}
		}
	}
	else
	{
		UE_LOGF(LogMIDIDevice, Warning, "MIDI device manager not initialized");
	}
}

void UMIDIDeviceManager::GetDefaultMIDIInputDeviceID(int32& DeviceID)
{
	if (bIsInitialized)
	{
		DeviceID = Pm_GetDefaultInputDeviceID();
	}
	else
	{
		UE_LOGF(LogMIDIDevice, Warning, "MIDI device manager not initialized");
	}
}

void UMIDIDeviceManager::GetMIDIOutputDeviceIDByName(const FString DeviceName, int32& DeviceID)
{
	if (bIsInitialized)
	{
		for (int32 i = 0; i < MIDIOutputDevicesInfo.Num(); ++i)
		{
			const FMIDIDeviceInfo& MIDIOutputDevice = MIDIOutputDevicesInfo[i];

			if (MIDIOutputDevice.DeviceName == DeviceName)
			{
				DeviceID = MIDIOutputDevice.DeviceID;
			}
		}
	}
	else
	{
		UE_LOGF(LogMIDIDevice, Warning, "MIDI device manager not initialized");
	}
}

void UMIDIDeviceManager::GetDefaultMIDIOutputDeviceID(int32& DeviceID)
{
	if (bIsInitialized)
	{
		DeviceID = Pm_GetDefaultOutputDeviceID();
	}
	else
	{
		UE_LOGF(LogMIDIDevice, Warning, "MIDI device manager not initialized");
	}
}

UMIDIDeviceController* UMIDIDeviceManager::CreateMIDIDeviceController(const int32 DeviceID, const int32 MIDIBufferSize)
{
	UMIDIDeviceController* NewMIDIDeviceController = nullptr;

	if (bIsInitialized)
	{
		// Create the MIDI Device Controller object.  It will be transient.
		NewMIDIDeviceController = NewObject<UMIDIDeviceController>();

		bool bStartedSuccessfully = false;
		NewMIDIDeviceController->StartupDevice(DeviceID, MIDIBufferSize, /* Out */ bStartedSuccessfully);

		if (!bStartedSuccessfully)
		{
			// Kill it
			NewMIDIDeviceController->MarkAsGarbage();
			NewMIDIDeviceController = nullptr;

			UE_LOGF(LogMIDIDevice, Warning, "Create MIDI Device Controller wasn't able to create the controller successfully. Returning a null reference.");
		}
	}
	else
	{
		UE_LOGF(LogMIDIDevice, Warning, "Create MIDI Device Controller isn't able to create a controller because the MIDI Device Manager failed to initialize. Look earlier in the log to see why it failed to startup.  Returning a null reference.");
	}

	return NewMIDIDeviceController;
}

UMIDIDeviceInputController* UMIDIDeviceManager::CreateMIDIDeviceInputController(const int32 DeviceID, const int32 MIDIBufferSize)
{
	UMIDIDeviceInputController* NewMIDIDeviceController = nullptr;

	// Create the MIDI Device Controller object.  It will be transient.
	NewMIDIDeviceController = NewObject<UMIDIDeviceInputController>();

	bool bStartedSuccessfully = false;
	NewMIDIDeviceController->StartupDevice(DeviceID, MIDIBufferSize, /* Out */ bStartedSuccessfully);

	if (!bStartedSuccessfully)
	{
		// Kill it
		NewMIDIDeviceController->MarkAsGarbage();
		NewMIDIDeviceController = nullptr;

		UE_LOGF(LogMIDIDevice, Warning, "Create MIDI Device Controller wasn't able to create the controller successfully. Returning a null reference.");
	}

	return NewMIDIDeviceController;
}

UMIDIDeviceOutputController* UMIDIDeviceManager::CreateMIDIDeviceOutputController(const int32 DeviceID)
{
	UMIDIDeviceOutputController* NewMIDIDeviceController = nullptr;

	// Create the MIDI Device Controller object.  It will be transient.
	NewMIDIDeviceController = NewObject<UMIDIDeviceOutputController>();

	bool bStartedSuccessfully = false;
	
	// BufferSize not used, but specify because of IMIDIDeviceControllerInterface
	NewMIDIDeviceController->StartupDevice(DeviceID, 1024, /* Out */ bStartedSuccessfully);

	if (!bStartedSuccessfully)
	{
		// Kill it
		NewMIDIDeviceController->MarkAsGarbage();
		NewMIDIDeviceController = nullptr;

		UE_LOGF(LogMIDIDevice, Warning, "Create MIDI Device Controller wasn't able to create the controller successfully. Returning a null reference.");
	}

	return NewMIDIDeviceController;
}

#undef LOCTEXT_NAMESPACE

