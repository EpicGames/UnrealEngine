// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "MIDIDeviceLog.h"
#include "MIDIDeviceManager.h"
#include "Tickable.h"


DEFINE_LOG_CATEGORY( LogMIDIDevice );


class FMIDIDeviceModule : public IModuleInterface, public FTickableGameObject
{

public:

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;


	// FTickableGameObject interface
	virtual void Tick( float DeltaTime ) override;

	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}
	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT( UMIDIDeviceManager, STATGROUP_Tickables );
	}
	virtual bool IsTickableWhenPaused() const
	{
		return true;
	}
	virtual bool IsTickableInEditor() const
	{
		return true;
	}


private:

	/** The last frame number we were ticked.  We don't want to tick multiple times per frame */
	uint32 LastFrameNumberWeTicked;

};


IMPLEMENT_MODULE( FMIDIDeviceModule, MIDIDevice )



void FMIDIDeviceModule::StartupModule()
{
	this->LastFrameNumberWeTicked = INDEX_NONE;

	UMIDIDeviceManager::StartupMIDIDeviceManager();
}


void FMIDIDeviceModule::ShutdownModule()
{
	UMIDIDeviceManager::ShutdownMIDIDeviceManager();
}


void FMIDIDeviceModule::Tick( float DeltaTime )
{
	if( LastFrameNumberWeTicked != GFrameCounter )
	{
		// Update the MIDI device manager
		UMIDIDeviceManager::ProcessMIDIEvents();

		LastFrameNumberWeTicked = GFrameCounter;
	}
}


