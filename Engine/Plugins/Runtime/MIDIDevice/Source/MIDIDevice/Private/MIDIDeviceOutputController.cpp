// Copyright Epic Games, Inc. All Rights Reserved.

#include "MIDIDeviceOutputController.h"
#include "MIDIDeviceLog.h"
#include "portmidi.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MIDIDeviceOutputController)

UMIDIDeviceOutputController::~UMIDIDeviceOutputController()
{
	// Clean everything up before we're garbage collected
	UMIDIDeviceOutputController::ShutdownDevice();
}

void UMIDIDeviceOutputController::StartupDevice(const int32 InitDeviceID, const int32 InitMIDIBufferSize, bool& bOutWasSuccessful)
{
	bOutWasSuccessful = false;

	this->DeviceID = InitDeviceID;
	this->PMMIDIStream = nullptr;

	const PmDeviceID PMDeviceID = this->DeviceID;
	const PmDeviceInfo* PMDeviceInfo = Pm_GetDeviceInfo(PMDeviceID);
	if(PMDeviceInfo != nullptr)
	{
		// Is the device already in use?  If so, spit out a warning
		if(PMDeviceInfo->opened != 0)
		{
			UE_LOGF(LogMIDIDevice, Warning, "Failed to bind to MIDI device '%ls' (ID: %i): Device is already in use", ANSI_TO_TCHAR(PMDeviceInfo->name), PMDeviceID);
			return;
		}

		if(PMDeviceInfo->output == 0)
		{
			UE_LOGF(LogMIDIDevice, Warning, "Failed to bind to MIDI device '%ls' (ID: %i): Device not setup to receive MIDI", ANSI_TO_TCHAR(PMDeviceInfo->name), PMDeviceID);
			return;
		}

		// @todo midi: Add options for timing/latency (see timeproc, and pm_Synchronize)

		const PmError PMError = Pm_OpenOutput(&this->PMMIDIStream, PMDeviceID, nullptr, 1, nullptr, nullptr, 0);
		if (PMError == pmNoError)
		{
			check(this->PMMIDIStream != nullptr);

			this->DeviceName = ANSI_TO_TCHAR(PMDeviceInfo->name);

			// Good to go!
			bOutWasSuccessful = true;
		}
		else
		{
			this->PMMIDIStream = nullptr;
			const FString ErrorText = MIDIDeviceInternal::ParsePmError(PMError);
			UE_LOGF(LogMIDIDevice, Error, "Unable to open output connection to MIDI device ID %i (%ls) (PortMidi error: %ls).", PMDeviceID, ANSI_TO_TCHAR(PMDeviceInfo->name), *ErrorText);
		}
	}
	else
	{
		UE_LOGF(LogMIDIDevice, Error, "Unable to query information about MIDI device (PortMidi device ID: %i).", PMDeviceID);
	}
}

void UMIDIDeviceOutputController::ShutdownDevice()
{
	if (this->PMMIDIStream != nullptr)
	{
		const PmError PMError = Pm_Close(this->PMMIDIStream);
		if (PMError != pmNoError)
		{
			const FString ErrorText = MIDIDeviceInternal::ParsePmError(PMError);
			UE_LOGF(LogMIDIDevice, Error, "Encounter an error when closing the output connection to MIDI device ID %i (%ls) (PortMidi error: %ls).", this->DeviceID, *this->DeviceName, *ErrorText);
		}

		this->PMMIDIStream = nullptr;
	}
}

void UMIDIDeviceOutputController::SendMIDIEvent(EMIDIEventType EventType, int32 Channel, int32 data1, int32 data2)
{
	if (this->PMMIDIStream != nullptr)
	{
		const int32 Status = ((int32)EventType << 4) | Channel;

		// timestamp is ignored because latency is set to 0
		Pm_WriteShort(this->PMMIDIStream, 0, Pm_Message(Status, data1, data2));
	}
}

void UMIDIDeviceOutputController::SendMIDINoteOn(int32 Channel, int32 Note, int32 Velocity)
{
	SendMIDIEvent(EMIDIEventType::NoteOn, Channel, Note, Velocity);
}

void UMIDIDeviceOutputController::SendMIDINoteOff(int32 Channel, int32 Note, int32 Velocity)
{
	SendMIDIEvent(EMIDIEventType::NoteOff, Channel, Note, Velocity);
}

void UMIDIDeviceOutputController::SendMIDIPitchBend(int32 Channel, int32 Pitch)
{
	Pitch = FMath::Clamp<int32>(Pitch, 0, 16383);

	SendMIDIEvent(EMIDIEventType::PitchBend, Channel, Pitch & 0x7F, Pitch >> 7);
}

void UMIDIDeviceOutputController::SendMIDINoteAftertouch(int32 Channel, int32 Note, float Amount)
{
	SendMIDIEvent(EMIDIEventType::NoteAfterTouch, Channel, Note, Amount);
}

void UMIDIDeviceOutputController::SendMIDIControlChange(int32 Channel, int32 Type, int32 Value)
{
	SendMIDIEvent(EMIDIEventType::ControlChange, Channel, Type, Value);
}

void UMIDIDeviceOutputController::SendMIDIProgramChange(int32 Channel, int32 ProgramNumber)
{
	SendMIDIEvent(EMIDIEventType::ProgramChange, Channel, ProgramNumber, 0);
}

void UMIDIDeviceOutputController::SendMIDIChannelAftertouch(int32 Channel, float Amount)
{
	SendMIDIEvent(EMIDIEventType::ChannelAfterTouch, Channel, Amount, 0);
}

