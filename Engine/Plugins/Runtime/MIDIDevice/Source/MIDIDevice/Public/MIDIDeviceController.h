// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MIDIDeviceControllerBase.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"


#include "MIDIDeviceController.generated.h"

#define UE_API MIDIDEVICE_API


UENUM(BlueprintType)
enum class EMIDIEventType : uint8
{
	/** Unrecognized MIDI event type.  You can look at Raw Event Type to see what it is. */
	Unknown			UMETA(Hidden),

	/** Note is released.  Velocity will contain the key pressure for devices that support that. */
	NoteOff = 8,

	/** Note is pressed down.  Velocity will contain the key pressure for devices that support that. */
	NoteOn = 9,

	/** Polyphonic key pressure.  This is sent after a key 'bottoms out' for devices that support it.  Velocity will contain the pressure value. */
	NoteAfterTouch = 10,

	/** This is sent for things like pedals when their controller state changes.  Velocity will contain the new value for the controller.  This event also is used for 'Channel Mode Changes' (Channels between 120-127), which encompass a variety of different features.  For those events, you'll need to interpret the values yourself. */
	ControlChange = 11,

	/** This is sent for some devices that support changing patches.  Velocity is usually ignored */
	ProgramChange = 12,

	/** Channel pressure value.  This is sent after a channel button 'bottoms out' for devices that support it.  Velocity will contain the pressure value. */
	ChannelAfterTouch = 13,

	/** For devices with levers or wheels, this indicates a change of state.  The data is interpreted a bit differently here, where the new value is actually 14-bits that contained within both the Control ID and Velocity */
	// @todo midi: Ideally set velocity to correct values in this case so Blueprints don't have to
	PitchBend = 14,
};


/** Callback function for received MIDI events */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_SevenParams(FOnMIDIEvent, class UMIDIDeviceController*, MIDIDeviceController, int32, Timestamp, EMIDIEventType, EventType, int32, Channel, int32, ControlID, int32, Velocity, int32, RawEventType);


UCLASS(MinimalAPI, BlueprintType)
class UMIDIDeviceController : public UMIDIDeviceControllerBase
{
	GENERATED_BODY()

public:

	/** Destructor that shuts down the device if it's still in use */
	UE_API virtual ~UMIDIDeviceController();

	/** Register with this to find out about incoming MIDI events from this device */
	UPROPERTY(BlueprintAssignable, Category="MIDI Device Controller")
	FOnMIDIEvent OnMIDIEvent;

	/** Called from UMIDIDeviceManager after the controller is created to get it ready to use.  Don't call this directly. */
	UE_API virtual void StartupDevice(const int32 InitDeviceID, const int32 InitMIDIBufferSize, bool& bOutWasSuccessful) override;

	/** Called every frame by UMIDIDeviceManager to poll for new MIDI events and broadcast them out to subscribers of OnMIDIEvent.  Don't call this directly. */
	UE_API virtual void ProcessIncomingMIDIEvents() override;

	/** Called during destruction to clean up this device.  Don't call this directly. */
	UE_API virtual void ShutdownDevice() override;

	/** The name of this device.  This name comes from the MIDI hardware, any might not be unique */
    virtual FString GetDeviceName() const override { return DeviceName; }

    /** Size of the MIDI buffer in bytes */
    virtual int32 GetMIDIBufferSize() const override { return MIDIBufferSize; }

protected:
	/** The unique ID of this device */
	UPROPERTY(BlueprintReadOnly, Category="MIDI Device Controller")
	int32 DeviceID;

	/** The name of this device.  This name comes from the MIDI hardware, any might not be unique */
	UPROPERTY(BlueprintReadOnly, Category = "MIDI Device Controller")
	FString DeviceName;

	/** The PortMidi stream used for MIDI input for this device */
	void* PMMIDIInputStream;

	/** Size of the MIDI buffer in bytes */
	int32 MIDIBufferSize;
};

#undef UE_API
