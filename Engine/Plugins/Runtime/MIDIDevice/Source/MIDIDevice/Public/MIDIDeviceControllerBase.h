// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MIDIDeviceControllerBase.generated.h"

#define UE_API MIDIDEVICE_API

/** Common functionality for the different MIDI Device Controllers. */
UCLASS(MinimalAPI, Abstract)
class UMIDIDeviceControllerBase : public UObject
{
	GENERATED_BODY()

public:
	/** Called from UMIDIDeviceManager after the controller is created to get it ready to use.  Don't call this directly. */
	virtual void StartupDevice(const int32 InitDeviceID, const int32 InitMIDIBufferSize, bool& bOutWasSuccessful) PURE_VIRTUAL(UMIDIDeviceControllerBase::StartupDevice,;)

	/** Called every frame by UMIDIDeviceManager to poll for new MIDI events and broadcast them out to subscribers of OnMIDIEvent.  Don't call this directly. */
	virtual void ProcessIncomingMIDIEvents() { };

	/** Called during destruction to clean up this device.  Don't call this directly. */
	virtual void ShutdownDevice() PURE_VIRTUAL(UMIDIDeviceControllerBase::ShutdownDevice,;)

	/** The name of this device.  This name comes from the MIDI hardware, any might not be unique */
	virtual FString GetDeviceName() const PURE_VIRTUAL(UMIDIDeviceControllerBase::GetDeviceName, return TEXT("");)

	/** Size of the MIDI buffer in bytes */
	virtual int32 GetMIDIBufferSize() const PURE_VIRTUAL(UMIDIDeviceControllerBase::GetMIDIBufferSize, return 0;)
};

#undef UE_API
