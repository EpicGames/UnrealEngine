// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MIDIDeviceControllerBase.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

#include "MIDIDeviceInputController.generated.h"

#define UE_API MIDIDEVICE_API


/** Callback function for received MIDI Note On event */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams(FOnMIDINoteOn, class UMIDIDeviceInputController*, MIDIDeviceController, int32, Timestamp, int32, Channel, int32, Note, int32, Velocity);

/** Callback function for received MIDI Note Off event */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams(FOnMIDINoteOff, class UMIDIDeviceInputController*, MIDIDeviceController, int32, Timestamp, int32, Channel, int32, Note, int32, Velocity);

/** Callback function for received MIDI Pitch Bend event */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams(FOnMIDIPitchBend, class UMIDIDeviceInputController*, MIDIDeviceController, int32, Timestamp, int32, Channel, int32, Pitch);

/** Callback function for received MIDI Aftertouch event */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams(FOnMIDIAftertouch, class UMIDIDeviceInputController*, MIDIDeviceController, int32, Timestamp, int32, Channel, int32, Note, int32, Amount);

/** Callback function for received MIDI Control Change event */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams(FOnMIDIControlChange, class UMIDIDeviceInputController*, MIDIDeviceController, int32, Timestamp, int32, Channel, int32, Type, int32, Value);

/** Callback function for received MIDI Program Change event */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams(FOnMIDIProgramChange, class UMIDIDeviceInputController*, MIDIDeviceController, int32, Timestamp, int32, Channel, int32, ControlID, int32, Velocity);

/** Callback function for received MIDI Channel Aftertouch event */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams(FOnMIDIChannelAftertouch, class UMIDIDeviceInputController*, MIDIDeviceController, int32, Timestamp, int32, Channel, int32, Amount);

/** Callback function for any kind of MIDI Event */
DECLARE_MULTICAST_DELEGATE_SixParams(FOnMIDIRawEvent, class UMIDIDeviceInputController* /* MIDIDeviceController */, int32 /* Timestamp */, int32 /* Type */, int32 /* Channel */, int32 /* MessageData1 */, int32 /* MessageData2 */);


UCLASS(MinimalAPI, BlueprintType)
class UMIDIDeviceInputController : public UMIDIDeviceControllerBase
{
	GENERATED_BODY()

public:

	/** Destructor that shuts down the device if it's still in use */
	UE_API virtual ~UMIDIDeviceInputController();

	/** Register with this to receive incoming MIDI Note On events from this device */
	UPROPERTY(BlueprintAssignable, Category = "MIDI Device Input Controller")
	FOnMIDINoteOn OnMIDINoteOn;

	/** Register with this to receive incoming MIDI Note Off events from this device */
	UPROPERTY(BlueprintAssignable, Category = "MIDI Device Input Controller")
	FOnMIDINoteOff OnMIDINoteOff;
	
	/** Register with this to receive incoming MIDI Pitch Bend events from this device */
	UPROPERTY(BlueprintAssignable, Category = "MIDI Device Input Controller")
	FOnMIDIPitchBend OnMIDIPitchBend;

	/** Register with this to receive incoming MIDI Aftertouch events from this device */
	UPROPERTY(BlueprintAssignable, Category = "MIDI Device Input Controller")
	FOnMIDIAftertouch OnMIDIAftertouch;

	/** Register with this to receive incoming MIDI Control Change events from this device */
	UPROPERTY(BlueprintAssignable, Category = "MIDI Device Input Controller")
	FOnMIDIControlChange OnMIDIControlChange;

	/** Register with this to receive incoming MIDI Program Change events from this device */
	UPROPERTY(BlueprintAssignable, Category = "MIDI Device Input Controller")
	FOnMIDIProgramChange OnMIDIProgramChange;

	/** Register with this to receive incoming MIDI Channel Aftertouch events from this device */
	UPROPERTY(BlueprintAssignable, Category = "MIDI Device Input Controller")
	FOnMIDIChannelAftertouch OnMIDIChannelAftertouch;

	/** Register with this to receive all of the MIDI events from this device */
	FOnMIDIRawEvent OnMIDIRawEvent;

	/** Called from UMIDIDeviceManager after the controller is created to get it ready to use.  Don't call this directly. */
	UE_API virtual void StartupDevice(const int32 InitDeviceID, const int32 InitMIDIBufferSize, bool& bOutWasSuccessful) override;

	/** Called every frame by UMIDIDeviceManager to poll for new MIDI events and broadcast them out to subscribers of OnMIDIEvent.  Don't call this directly. */
	UE_API virtual void ProcessIncomingMIDIEvents() override;

	/** Called during destruction to clean up this device.  Don't call this directly. */
	UE_API virtual void ShutdownDevice() override;

	/** The name of this device.  This name comes from the MIDI hardware, any might not be unique */
	virtual FString GetDeviceName() const override { return DeviceName; }

	/** Size of the MIDI buffer in bytes */
	virtual int32 GetMIDIBufferSize() const override { return MIDIBufferSize; }

protected:
	/** The unique ID of this device */
	UPROPERTY(BlueprintReadOnly, Category="MIDI Device Input Controller")
	int32 DeviceID;

	/** The name of this device.  This name comes from the MIDI hardware, any might not be unique */
	UPROPERTY(BlueprintReadOnly, Category = "MIDI Device Input Controller")
	FString DeviceName;

	/** The PortMidi stream used for MIDI input for this device */
	void* PMMIDIStream;

	/** Size of the MIDI buffer in bytes */
	int32 MIDIBufferSize;
};

#undef UE_API
