// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MIDIDeviceInputController.h"
#include "MIDIDeviceOutputController.h"

#include "MIDIDeviceManager.generated.h"

#define UE_API MIDIDEVICE_API

USTRUCT(BlueprintType)
struct FFoundMIDIDevice
{
	GENERATED_BODY()

	/** The unique ID of this MIDI device */
	UPROPERTY(BlueprintReadOnly, Category="MIDI Device Manager")
	int32 DeviceID = 0;

	/** The name of this device.  This name comes from the MIDI hardware, and might not be unique */
	UPROPERTY(BlueprintReadOnly, Category="MIDI Device Manager")
	FString DeviceName;

	/** True if the device supports sending events to us */
	UPROPERTY(BlueprintReadOnly, Category="MIDI Device Manager")
	bool bCanReceiveFrom = false;

	/** True if the device supports receiving events from us */
	UPROPERTY(BlueprintReadOnly, Category="MIDI Device Manager")
	bool bCanSendTo = false;

	/** Whether the device is already in use.  You might not want to create a controller for devices that are busy.  Someone else could be using it. */
	UPROPERTY(BlueprintReadOnly, Category="MIDI Device Manager")
	bool bIsAlreadyInUse = false;

	/** True if this is the default MIDI device for input on this system */
	UPROPERTY(BlueprintReadOnly, Category="MIDI Device Manager")
	bool bIsDefaultInputDevice = false;

	/** True if this is the default MIDI device for output on this system */
	UPROPERTY(BlueprintReadOnly, Category="MIDI Device Manager")
	bool bIsDefaultOutputDevice = false;
};


USTRUCT(BlueprintType)
struct FMIDIDeviceInfo
{
	GENERATED_BODY()

	/** The unique ID of this MIDI device */
	UPROPERTY(BlueprintReadOnly, Category = "MIDI Device Manager")
	int32 DeviceID = 0;

	/** The name of this device.  This name comes from the MIDI hardware, any might not be unique */
	UPROPERTY(BlueprintReadOnly, Category = "MIDI Device Manager")
	FString DeviceName;

	/** Whether the device is already in use.  You might not want to create a controller for devices that are busy.  Someone else could be using it. */
	UPROPERTY(BlueprintReadOnly, Category = "MIDI Device Manager")
	bool bIsAlreadyInUse = false;

	/** True if this is the default MIDI device for input on this system */
	UPROPERTY(BlueprintReadOnly, Category = "MIDI Device Manager")
	bool bIsDefaultDevice = false;
};

UCLASS(MinimalAPI)
class UMIDIDeviceManager : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Shuts down any existing midi devices to ensure no dangling streams are left, and mark any input or output controller for cleanup
	 */
	UFUNCTION(BlueprintCallable, Category="MIDI Device Manager")
	static UE_API void ShutDownAllMIDIDevices();
	/**
	 * Enumerates all of the connected MIDI devices and reports back with the IDs and names of those devices.  This operation is a little expensive
	 * so only do it once at startup, or if you think that a new device may have been connected.
	 *
	 * @param	OutMIDIDevices	A list of available MIDI devices
	 */
	UFUNCTION(BlueprintCallable, Category="MIDI Device Manager")
	static UE_API void FindMIDIDevices(TArray<FFoundMIDIDevice>& OutMIDIDevices);

	/**
	 * Enumerates all of the MIDI input and output devices and reports back useful infos such as IDs and names of those devices. This operation is a little expensive
	 * so only do it once at startup, or if you think that a new device may have been connected.
	 *
	 * @param 	OutMIDIInputDevices		A list of available MIDI Input devices
	 * @param 	OutMIDIOutputDevices	A list of available MIDI Output devices
	 */
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Manager")
	static UE_API void FindAllMIDIDeviceInfo(TArray<FMIDIDeviceInfo>& OutMIDIInputDevices, TArray<FMIDIDeviceInfo>& OutMIDIOutputDevices);

	/**
	 * Retrieves the MIDI input device ID by name. Call "Find All MIDI Device Info" beforehand to enumerate the available input devices. 
	 *
	 * @param	DeviceName		The Name of the MIDI device you want to talk to.
	 * @param	DeviceID		The Device ID of the MIDI device with that name.
	 */
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Manager")
	static UE_API void GetMIDIInputDeviceIDByName(const FString DeviceName, int32& DeviceID);

	/**
	 * Retrieves the default MIDI input device ID. Call "Find All MIDI Device Info" beforehand to enumerate the available input devices.
	 *
	 * @param	DeviceID		The Device ID of the MIDI input device with that name.
	 */
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Manager")
	static UE_API void GetDefaultMIDIInputDeviceID(int32& DeviceID);

	/**
	 * Retrieves the MIDI output device ID by name. Call "Find All MIDI Device Info" beforehand to enumerate the available output devices.
	 *
	 * @param	DeviceName		The Name of the MIDI device you want to talk to.
	 * @param	DeviceID		The Device ID of the MIDI output device associated with that name.
	 */
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Manager")
	static UE_API void GetMIDIOutputDeviceIDByName(const FString DeviceName, int32& DeviceID);

	/**
	 * Retrieves the default MIDI output device ID. Call "Find All MIDI Device Info" beforehand to enumerate the available input devices.
	 *
	 * @param	DeviceID		The Device ID of the MIDI output device with that name.
	 */
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Manager")
	static UE_API void GetDefaultMIDIOutputDeviceID(int32& DeviceID);

	/**
	 * Creates an instance of a MIDI device controller that can be used to interact with a connected MIDI device
	 *
	 * @param	DeviceID		The ID of the MIDI device you want to talk to.  Call "Find MIDI Devices" to enumerate the available devices.
	 * @param	MIDIBufferSize	How large the buffer size (in number of MIDI events) should be for incoming MIDI data.  Larger values can incur higher latency costs for incoming events, but don't set it too low or you'll miss events and your stuff will sound bad.
	 *
	 * @return	If everything goes okay, a valid MIDI device controller object will be returned.  If anything goes wrong, a null reference will be returned.
	 */
	UFUNCTION(BlueprintCallable, Category="MIDI Device Manager")
	static UE_API class UMIDIDeviceController* CreateMIDIDeviceController(const int32 DeviceID, const int32 MIDIBufferSize = 1024);

	/**
	 * Creates an instance of a MIDI device controller that can be used to interact with a connected MIDI device
	 *
	 * @param	DeviceID		The ID of the MIDI device you want to talk to.  Call "Find MIDI Devices" to enumerate the available devices.
	 * @param	MIDIBufferSize	How large the buffer size (in number of MIDI events) should be for incoming MIDI data.  Larger values can incur higher latency costs for incoming events, but don't set it too low or you'll miss events and your stuff will sound bad.
	 *
	 * @return	If everything goes okay, a valid MIDI device controller object will be returned.  If anything goes wrong, a null reference will be returned.
	 */
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Manager")
	static UE_API class UMIDIDeviceInputController* CreateMIDIDeviceInputController(const int32 DeviceID, const int32 MIDIBufferSize = 1024);

	/**
	 * Creates an instance of a MIDI output device controller that can be used to interact with a connected MIDI device
	 *
	 * @param	DeviceID		The ID of the MIDI device you want to talk to.  Call "Find MIDI Devices" to enumerate the available devices.
	 *
	 * @return	If everything goes okay, a valid MIDI device controller object will be returned.  If anything goes wrong, a null reference will be returned.
	 */
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Manager")
	static UE_API class UMIDIDeviceOutputController* CreateMIDIDeviceOutputController(const int32 DeviceID);

	/** Called from FMIDIDeviceModule to startup the device manager.  Don't call this yourself. */
	static UE_API void StartupMIDIDeviceManager();

	/** Called from FMIDIDeviceModule to shutdown the device manager.  Don't call this yourself. */
	static UE_API void ShutdownMIDIDeviceManager();

	/** Called every frame to look for any new MIDI events that were received, and routes those events to subscribers.  Don't call this yourself.
	    It will be called by FMIDIDeviceModule::Tick(). */
	static UE_API void ProcessMIDIEvents();


private:
	/** Called internally to update device info.  */
	static void ReinitializeDeviceManager();

	/** Implementation of FindMIDIDevices that does't reinitialize the Device Manager. */
	static void FindMIDIDevicesInternal(TArray<FFoundMIDIDevice>& OutMIDIDevices);

	/** Implementation of FindMIDIDeviceInfo that does't reinitialize the Device Manager. */
	static void FindAllMIDIDeviceInfoInternal(TArray<FMIDIDeviceInfo>& OutMIDIInputDevices, TArray<FMIDIDeviceInfo>& OutMIDIOutputDevices);

private:
	/** True if everything is initialized OK */
	static bool bIsInitialized;

	/** MIDI input devices info */
	static TArray<FMIDIDeviceInfo> MIDIInputDevicesInfo;

	/** MIDI output devices info */
	static TArray<FMIDIDeviceInfo> MIDIOutputDevicesInfo;
};

#undef UE_API
