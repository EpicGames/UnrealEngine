// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MIDIDeviceController.h"
#include "MIDIDeviceControllerBase.h"
#include "UObject/Object.h"

#include "MIDIDeviceOutputController.generated.h"

#define UE_API MIDIDEVICE_API

UCLASS(MinimalAPI, BlueprintType)
class UMIDIDeviceOutputController : public UMIDIDeviceControllerBase
{
	GENERATED_BODY()

public:

	/** Destructor that shuts down the device if it's still in use */
	UE_API virtual ~UMIDIDeviceOutputController();

	/** Sends MIDI event raw data for an event type 
	*
	* @param	EventType		The event type as specified in the EMIDIEventType struct
	* @param	Channel			The MIDI channel to send 
	* @param	Data1			The first part of the MIDI data
	* @param	Data2			The second part of the MIDI data
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Output Controller")
	UE_API void SendMIDIEvent(EMIDIEventType EventType, int32 Channel, int32 data1, int32 data2);

	/** Sends MIDI Note On event type 
	*
	* @param	Channel			The MIDI channel to send
	* @param	Note			The MIDI Note value
	* @param	Velocity		The MIDI Velocity value
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Output Controller")
	UE_API void SendMIDINoteOn(int32 Channel, int32 Note, int32 Velocity);

	/** Sends MIDI Note Off event type 
	*
	* @param	Channel			The MIDI channel to send
	* @param	Note			The MIDI Note value
	* @param	Velocity		The MIDI Velocity value
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Output Controller")
	UE_API void SendMIDINoteOff(int32 Channel, int32 Note, int32 Velocity);

	/** Sends MIDI Pitch Bend event type
	*
	* @param	Channel			The MIDI channel to send
	* @param	Pitch			The MIDI Pitch Bend value (0-16383)
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Output Controller")
	UE_API void SendMIDIPitchBend(int32 Channel, int32 Pitch);

	/** Sends MIDI Note Aftertouch event type 
	*
	* @param	Channel			The MIDI channel to send
	* @param	Note			The MIDI Note value
	* @param	Amount			The MIDI aftertouch amount
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Output Controller")
	UE_API void SendMIDINoteAftertouch(int32 Channel, int32 Note, float Amount);

	/** Sends MIDI Control Change event type 
	*
	* @param	Channel			The MIDI channel to send
	* @param	Type			The MIDI control type change
	* @param	Value			The MIDI Value for the control change
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Output Controller")
	UE_API void SendMIDIControlChange(int32 Channel, int32 Type, int32 Value);

	/** Sends MIDI Program Change event type 
	*
	* @param	Channel				The MIDI channel to send
	* @param	ProgramNumberType	The MIDI Program Number value
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Output Controller")
	UE_API void SendMIDIProgramChange(int32 Channel, int32 ProgramNumber);

	/** Sends MIDI Channel Aftertouch event type 
	*
	* @param	Channel				The MIDI channel to send
	* @param	Amount				The MIDI Amount of aftertouch 
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "MIDI Device Output Controller")
	UE_API void SendMIDIChannelAftertouch(int32 Channel, float Amount);


	/** Called from UMIDIDeviceManager after the controller is created to get it ready to use.  Don't call this directly. */
	UE_API virtual void StartupDevice(const int32 InitDeviceID, const int32 InitMIDIBufferSize, bool& bOutWasSuccessful) override;

	/** Called during destruction to clean up this device.  Don't call this directly. */
	UE_API virtual void ShutdownDevice() override;

	/** The name of this device.  This name comes from the MIDI hardware, any might not be unique */
	virtual FString GetDeviceName() const override { return DeviceName; }

	/** Size of the MIDI buffer in bytes */
	virtual int32 GetMIDIBufferSize() const override { return 0; }

protected:

	/** The unique ID of this device */
	UPROPERTY(BlueprintReadOnly, Category="MIDI Device Output Controller")
	int32 DeviceID;

	/** The name of this device.  This name comes from the MIDI hardware, any might not be unique */
	UPROPERTY(BlueprintReadOnly, Category = "MIDI Device Output Controller")
	FString DeviceName;

	/** The PortMidi stream used for MIDI output for this device */
	void* PMMIDIStream;
};

#undef UE_API
